{
	"FileVersion": 1,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "ChaosUserDataPT",
	"Description": "Custom per-particle userdata. Write-only on game thread, read-only on physics thread.",
	"Category": "Editor",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"CanContainContent": false,
	"IsBetaVersion": true,
	"Installed": false,
	"EnabledByDefault": true,
	"Modules": [
		{
			"Name": "ChaosUserDataPT",
			"Type": "Runtime",
			"LoadingPhase": "Default"
		}
	]
}