// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

namespace UnrealBuildTool.Rules
{
	public class ChaosUserDataPT : ModuleRules
	{

		public ChaosUserDataPT(ReadOnlyTargetRules Target) : base(Target)
		{
			PublicDependencyModuleNames.AddRange(
				new string[]
				{
					"Core",
				}
			);

			SetupModulePhysicsSupport(Target);
			PrivateDefinitions.Add("CHAOS_INCLUDE_LEVEL_1=1");
		}
	}
}
