// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosUserDataPTModule.h"
#include "Modules/ModuleManager.h"

void FChaosUserDataPTModule::StartupModule()
{
	// This code will execute after your module is loaded into memory; the exact timing is specified in the .uplugin file per-module
}

void FChaosUserDataPTModule::ShutdownModule()
{
	// This function may be called during shutdown to clean up your module.  For modules that support dynamic reloading,
	// we call this function before unloading the module.
}

IMPLEMENT_MODULE(FChaosUserDataPTModule, ChaosUserDataPT)
