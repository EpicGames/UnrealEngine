// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosUserDataPTStats.h"
#include "Stats/Stats.h"

namespace Chaos
{
	DEFINE_STAT(STAT_UserDataPT_SetData_GT);
	DEFINE_STAT(STAT_UserDataPT_RemoveData_GT);
	DEFINE_STAT(STAT_UserDataPT_GetData_PT);
	DEFINE_STAT(STAT_UserDataPT_GetData_GT);
	DEFINE_STAT(STAT_UserDataPT_UpdateData_PT);
	DEFINE_STAT(STAT_UserDataPT_RemoveData_PT);
	DEFINE_STAT(STAT_UserDataPT_ClearData_PT);
}
