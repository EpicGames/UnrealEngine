// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PhysicsProxy/SingleParticlePhysicsProxy.h"
#include "PBDRigidsSolver.h"
#include "Chaos/PhysicsObject.h"
#include "PhysicsEngine/PhysicsObjectExternalInterface.h"
#include "ChaosUserDataPTStats.h"
#include <limits>

/*

	Chaos User Data PT
	==================

	The idea behind this tool is to provide a generic way of associating
	custom data with physics particles, which is write-only from the game
	thread, and read-only from the physics thread.

	This comes in handy when physical interactions at the per-contact level
	need to be affected by gameplay properties.

	A TUserDataManagerPT is a sim callback which is templated on a type TUserData. It
	owns an instance of TUserData for each particle which has been used as an
	argument to SetData_GT.

	In order to use a TUserDataManagerPT, it will need to be created using the chaos
	solver's CreateAndRegisterSimCallbackObject_External method. This
	library does not natively provide a method of accessing the
	appropriate TUserDataManagerPT instance from the physics thread, but this can
	be achieved in a number of ways - it is left up to the game to decide
	how to do this for flexibility.

*/

namespace Chaos
{
	template <typename TUserData>
	struct TUserDataManagerPTInput;

	template <typename TUserData>
	class TUserDataManagerPT;

	// Configuration for TUserDataManagerPT behavior.
	// Passed at construction time and stored as a const member.
	struct FUserDataPTConfig
	{
		// When true, TUserDataManagerPT will cache userdata on GT as well as PT,
		// so that GT users can get the last set value. Useful when they want
		// to make incremental changes, but incurs a higher memory footprint.
		bool bGetData_GT = false;
	};

	// TUserDataManagerPTInput
	//
	// Input is a collection of pointers to new and updated userdata objects
	// to be sent to the physics thread
	//
	// NOTE: On deprecation strategy here.
	// The members of this struct should never have been public, so they have
	// all been deprecated. Private duplicates are now in use, but those will
	// be renamed to the original names once the deprecated versions are removed.
	//
	// Copies are needed so that during the deprecation phase the internal
	// TUserDataManagerPT code doesn't trigger deprecation warnings for valid
	// operations.
	template <typename TUserData>
	struct TUserDataManagerPTInput : public FSimCallbackInput
	{
		UE_DEPRECATED(5.8, "UserDataToAdd no longer used, will be removed.")
		mutable TMap<FUniqueIdx, TUniquePtr<TUserData>> UserDataToAdd;

		UE_DEPRECATED(5.8, "UserDataToRemove no longer used, will be removed.")
		TSet<FUniqueIdx> UserDataToRemove;

		UE_DEPRECATED(5.8, "bClear no longer used, will be removed.")
		bool bClear = false;

		UE_DEPRECATED(5.8, "bResizeOnClear no longer used, will be removed.")
		bool bResizeOnClear = false;

		UE_DEPRECATED(5.8, "Identifier no longer used, will be removed.")
		uint32 Identifier = std::numeric_limits<uint32>::max();

		void Reset()
		{
			UserDataToAddPrivate.Reset();
			UserDataToRemovePrivate.Reset();
			bClearPrivate = false;
			bResize = false;
			IdentifierPrivate = std::numeric_limits<uint32>::max();
		}

	private:
		// Map of particle unique indices to user data ptrs
		// 
		// NOTE: This is marked mutable because the userdata objects must be
		// moved to the internal array after making it
		// OnPreSimulate_Internal, but input objects are const in that
		// context. This might be frowned on, but since TUserDataManagerPTInput is a
		// class which is only used internally to TUserDataManagerPT, an argument
		// could be made either way.
		mutable TMap<FUniqueIdx, TUniquePtr<TUserData>> UserDataToAddPrivate;

		// Set of particle unique indices for which to remove user data
		TSet<FUniqueIdx> UserDataToRemovePrivate;

		// Flag for clearing all data
		bool bClearPrivate = false;

		// Flag for resizing container to fit data
		bool bResize = false;

		// Monotonically increasing identifier for the input object. Each
		// newly constructed input will store and increment this;
		uint32 IdentifierPrivate = std::numeric_limits<uint32>::max();

		// NOTE: In UE 5.8, mark all contents of this struct private
		friend class TUserDataManagerPT<TUserData>;
	};

	// TUserDataManagerPT
	//
	// A chaos callback object which stores and allows access to user data
	// associated with particles on the physics thread.
	//
	// Note that FSimCallbackOutput is the output struct - this carries no
	// data because this is a one-way callback. We use it basically just to
	// marshal data in one direction.
	template <typename TUserData>
	class TUserDataManagerPT : public TSimCallbackObject<
		TUserDataManagerPTInput<TUserData>,
		FSimCallbackNoOutput,
		ESimCallbackOptions::Presimulate | ESimCallbackOptions::ParticleUnregister>
	{
		using TInput = TUserDataManagerPTInput<TUserData>;

	public:

		explicit TUserDataManagerPT(const FUserDataPTConfig& InConfig = FUserDataPTConfig())
			: Config(InConfig)
		{
		}

		virtual ~TUserDataManagerPT() = default;

		// Add or update user data associated with a particle handle (copy)
		template <typename TParticleHandle>
		bool SetData_GT(const TParticleHandle& Handle, const TUserData& UserData)
		{
			TUserData Copy(UserData);
			return SetData_GT(Handle, MoveTemp(Copy));
		}

		// Add or update user data associated with a particle handle (move)
		template <typename TParticleHandle>
		bool SetData_GT(const TParticleHandle& Handle, TUserData&& UserData)
		{
			SCOPE_CYCLE_COUNTER(STAT_UserDataPT_SetData_GT);

			if (this->GetSolver() != nullptr)
			{
				if (TInput* Input = this->GetProducerInputData_External())
				{
					// Do GT data caching if configured
					if (Config.bGetData_GT)
					{
						UserDataMap_GT.EmplaceAt(Handle.UniqueIdx().Idx, MakeUnique<TUserData>(UserData));
					}

					// Add the data to the map to be sent to physics thread
					Input->UserDataToAddPrivate.Emplace(Handle.UniqueIdx(), MakeUnique<TUserData>(MoveTemp(UserData)));

					// In case it was removed and then added again in the same
					// frame, untrack this particle for data removal
					Input->UserDataToRemovePrivate.Remove(Handle.UniqueIdx());

					// If this is a new input, set it's IdentifierPrivate
					if (Input->IdentifierPrivate == std::numeric_limits<uint32>::max())
					{
						Input->IdentifierPrivate = InputIdentifier_GT++;
					}

					// Successfully queued for add/update
					return true;
				}
			}

			// Failed to queue for add/update
			return false;
		}

		// Get user data from the GT cache. Returns nullptr if
		// Config.bGetData_GT is false or the particle has no data.
		template <typename TParticleHandle>
		const TUserData* GetData_GT(const TParticleHandle& Handle) const
		{
			SCOPE_CYCLE_COUNTER(STAT_UserDataPT_GetData_GT);

			if (!Config.bGetData_GT)
			{
				return nullptr;
			}

			const int32 Idx = Handle.UniqueIdx().Idx;
			return UserDataMap_GT.IsValidIndex(Idx) ? UserDataMap_GT[Idx].Get() : nullptr;
		}

		// Remove user data associated with a particle handle
		template <typename TParticleHandle>
		bool RemoveData_GT(const TParticleHandle& Handle)
		{
			SCOPE_CYCLE_COUNTER(STAT_UserDataPT_RemoveData_GT);

			if (this->GetSolver() != nullptr)
			{
				if (TInput* Input = this->GetProducerInputData_External())
				{
					// Track the particle for removal. No point in doing so
					// if we've already marked all data for clearing.
					if (!Input->bClearPrivate)
					{
						Input->UserDataToRemovePrivate.Add(Handle.UniqueIdx());
					}

					// In case it was added/updated and then removed in the
					// same frame, untrack the add/update
					Input->UserDataToAddPrivate.Remove(Handle.UniqueIdx());

					// Remove GT data cache if configured
					if (Config.bGetData_GT)
					{
						if (UserDataMap_GT.IsValidIndex(Handle.UniqueIdx().Idx))
						{
							UserDataMap_GT.RemoveAt(Handle.UniqueIdx().Idx);
						}
					}

					// Successfully queued for removal
					return true;
				}
			}

			// Failed to queue for removal
			return false;
		}

		template <>
		bool SetData_GT<Chaos::FPhysicsObjectHandle>(const Chaos::FPhysicsObjectHandle& Object, const TUserData& UserData)
		{
			// Get the game thread particle handle and set UserData on that
			FLockedReadPhysicsObjectExternalInterface Interface = FPhysicsObjectExternalInterface::LockRead(Object);
			if (const Chaos::FGeometryParticle* Particle = Interface->GetParticle(Object))
			{
				return SetData_GT(*Particle, UserData);
			}

			return false;
		}

		template <>
		bool RemoveData_GT<Chaos::FPhysicsObjectHandle>(const Chaos::FPhysicsObjectHandle& Object)
		{
			// Get the game thread particle handle and remove UserData from it
			FLockedReadPhysicsObjectExternalInterface Interface = FPhysicsObjectExternalInterface::LockRead(Object);
			if (const Chaos::FGeometryParticle* Particle = Interface->GetParticle(Object))
			{
				return RemoveData_GT(*Particle);
			}

			return false;
		}

		// TParticleHandle is generalized here because it can be
		// FRigidBodyHandle_Internal or FGeometryParticleHandle which have
		// the same api...
		template <typename TParticleHandle>
		const TUserData* GetData_PT(const TParticleHandle& Handle) const
		{
			SCOPE_CYCLE_COUNTER(STAT_UserDataPT_GetData_PT);

			const int32 Idx = Handle.UniqueIdx().Idx;
			return UserDataMap_PT.IsValidIndex(Idx) ? UserDataMap_PT[Idx].Get() : nullptr;
		}

		// Execute a lambda on each data entry
		void VisitData_PT(TFunctionRef<void(Chaos::FUniqueIdx, const TUserData&)> Lambda)
		{
			for (auto It = UserDataMap_PT.CreateConstIterator(); It; ++It)
			{
				if (It->IsValid())
				{
					Lambda(Chaos::FUniqueIdx(It.GetIndex()), *It->Get());
				}
			}
		}

		UE_DEPRECATED(5.8, "VisitAllData_PT will be removed, use VisitData_PT instead.")
		void VisitAllData_PT(const TFunction<void(Chaos::FUniqueIdx, const TUserData&)>& Lambda)
		{
			VisitData_PT(Lambda);
		}

		// Clear all data
		bool ClearData_GT(const bool bResize = true)
		{
			if (TInput* Input = this->GetProducerInputData_External())
			{
				// All userdatas to add and remove can be cleared. Any userdatas that
				// are added after this point will still be added.
				Input->UserDataToAddPrivate.Empty();
				Input->UserDataToRemovePrivate.Empty();

				// Mark the flags for clearing all data and resizing container
				Input->bClearPrivate = true;
				Input->bResize = bResize;

				// If this is a new input, set it's IdentifierPrivate
				if (Input->IdentifierPrivate == std::numeric_limits<uint32>::max())
				{
					Input->IdentifierPrivate = InputIdentifier_GT++;
				}

				// Empty the GT data cache if configured
				if (Config.bGetData_GT)
				{
					UserDataMap_GT.Empty();

					if (bResize)
					{
						UserDataMap_GT.Shrink();
					}
				}

				// Succeeded
				return true;
			}

			// Failed to queue data for clearing 
			return false;
		}

		virtual FName GetFNameForStatId() const override
		{
			const static FLazyName StaticName("TUserDataManagerPT");
			return StaticName;
		}

		// Runtime configuration (immutable after construction)
		const FUserDataPTConfig Config;

	protected:

		virtual void OnPostInitialize_Internal() override
		{
			if (Config.bGetData_GT)
			{
				if (FPBDRigidsSolver* RigidsSolver = static_cast<FPBDRigidsSolver*>(this->GetSolver()))
				{
PRAGMA_DISABLE_INTERNAL_WARNINGS
					ParticleUnregisteredGTHandle = RigidsSolver->AddParticleUnregisteredGTCallback(
						FSolverParticleUnregisteredGT::FDelegate::CreateLambda(
						[this](FSingleParticlePhysicsProxy* /*Proxy*/, FUniqueIdx UniqueIdx)
						{
							if (UserDataMap_GT.IsValidIndex(UniqueIdx.Idx))
							{
								UserDataMap_GT.RemoveAt(UniqueIdx.Idx);
							}
						})
					);
PRAGMA_ENABLE_INTERNAL_WARNINGS
				}
			}
		}

		virtual void OnPreShutdown_External() override
		{
			if (ParticleUnregisteredGTHandle.IsValid())
			{
				if (FPBDRigidsSolver* RigidsSolver = static_cast<FPBDRigidsSolver*>(this->GetSolver()))
				{
PRAGMA_DISABLE_INTERNAL_WARNINGS
					RigidsSolver->RemoveParticleUnregisteredGTCallback(ParticleUnregisteredGTHandle);
					ParticleUnregisteredGTHandle.Reset();
PRAGMA_ENABLE_INTERNAL_WARNINGS
				}
			}
		}

		virtual void OnPreSimulate_Internal() override
		{
			if (const TInput* Input = this->GetConsumerInput_Internal())
			{
				// Only proceed if the input has not yet been processed.
				// 
				// It's possible that we'll get multiple presimulate calls
				// with the same input because the same input continues to be
				// provided until a new one is received, so we cache the
				// timestamp of the last processed input to make sure that we
				// don't double-process it.
				if (InputIdentifier_PT != Input->IdentifierPrivate)
				{
					InputIdentifier_PT = Input->IdentifierPrivate;

					// Clear all data
					if (Input->bClearPrivate)
					{
						SCOPE_CYCLE_COUNTER(STAT_UserDataPT_ClearData_PT);

						// Empty the userdata map
						UserDataMap_PT.Empty();
					}

					// Add new data
					if (Input->UserDataToAddPrivate.Num() > 0)
					{
						SCOPE_CYCLE_COUNTER(STAT_UserDataPT_UpdateData_PT);

						// Move all the user data to the internal map
						for (auto& Iter : Input->UserDataToAddPrivate)
						{
							UserDataMap_PT.EmplaceAt(Iter.Key.Idx, MoveTemp(Iter.Value));
						}
					}

					// Remove old data
					if (Input->UserDataToRemovePrivate.Num() > 0)
					{
						SCOPE_CYCLE_COUNTER(STAT_UserDataPT_RemoveData_PT);

						// Delete user data that has been removed
						for (const FUniqueIdx Idx : Input->UserDataToRemovePrivate)
						{
							if (UserDataMap_PT.IsValidIndex(Idx.Idx))
							{
								UserDataMap_PT.RemoveAt(Idx.Idx);
							}
						}

					}

					// Resize container
					if (Input->bResize)
					{
						// Shrink sparse array to fit the data
						UserDataMap_PT.Shrink();
					}
				}
			}
		}

		virtual void OnParticleUnregistered_Internal(TArray<TTuple<FUniqueIdx, FSingleParticlePhysicsProxy*>>& UnregisteredProxies) override
		{
			if (UnregisteredProxies.Num() > 0)
			{
				// If any particles were removed in this physics tick, check to see if they have
				// userdata that we were tracking and remove those userdata instances.
				for (TTuple<FUniqueIdx, FSingleParticlePhysicsProxy*> Proxy : UnregisteredProxies)
				{
					const FUniqueIdx Idx = Proxy.Get<FUniqueIdx>();
					if (UserDataMap_PT.IsValidIndex(Idx.Idx))
					{
						UserDataMap_PT.RemoveAt(Idx.Idx);
					}
				}

				// Shrink sparse array if we took elements off the end
				UserDataMap_PT.Shrink();
			}
		}

		// Identifier of the next input to be created on the game thread
		uint32 InputIdentifier_GT = 0;

		// Identifier of the last input to be consumed on the physics thread
		uint32 InputIdentifier_PT = std::numeric_limits<uint32>::max();

		// Map of particle unique ids to user data, for PT access
		TSparseArray<TUniquePtr<TUserData>> UserDataMap_PT;

		// Map of particle unique ids to user data, for optional GT access.
		// Only populated when Config.bGetData_GT is true; otherwise stays empty.
		TSparseArray<TUniquePtr<TUserData>> UserDataMap_GT;

		// Delegate handle for GT particle unregistration cleanup
		FDelegateHandle ParticleUnregisteredGTHandle;
	};
}
