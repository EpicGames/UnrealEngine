// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Stats/Stats.h"

DECLARE_STATS_GROUP(TEXT("ChaosUserDataPT"), STATGROUP_ChaosUserDataPT, STATCAT_Advanced);

namespace Chaos
{
	DECLARE_CYCLE_STAT_EXTERN(TEXT("ChaosUserDataPT::SetData_GT"), STAT_UserDataPT_SetData_GT, STATGROUP_ChaosUserDataPT, CHAOSUSERDATAPT_API);
	DECLARE_CYCLE_STAT_EXTERN(TEXT("ChaosUserDataPT::RemoveData_GT"), STAT_UserDataPT_RemoveData_GT, STATGROUP_ChaosUserDataPT, CHAOSUSERDATAPT_API);
	DECLARE_CYCLE_STAT_EXTERN(TEXT("ChaosUserDataPT::GetData_PT"), STAT_UserDataPT_GetData_PT, STATGROUP_ChaosUserDataPT, CHAOSUSERDATAPT_API);
	DECLARE_CYCLE_STAT_EXTERN(TEXT("ChaosUserDataPT::GetData_GT"), STAT_UserDataPT_GetData_GT, STATGROUP_ChaosUserDataPT, CHAOSUSERDATAPT_API);
	DECLARE_CYCLE_STAT_EXTERN(TEXT("ChaosUserDataPT::OnPreSimulate::UpdateData"), STAT_UserDataPT_UpdateData_PT, STATGROUP_ChaosUserDataPT, CHAOSUSERDATAPT_API);
	DECLARE_CYCLE_STAT_EXTERN(TEXT("ChaosUserDataPT::OnPreSimulate::RemoveData"), STAT_UserDataPT_RemoveData_PT, STATGROUP_ChaosUserDataPT, CHAOSUSERDATAPT_API);
	DECLARE_CYCLE_STAT_EXTERN(TEXT("ChaosUserDataPT::OnPreSimulate::ClearData"), STAT_UserDataPT_ClearData_PT, STATGROUP_ChaosUserDataPT, CHAOSUSERDATAPT_API);
}
